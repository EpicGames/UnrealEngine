{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkLens",
	"Description": "Adds a new LiveLink LensRole and LensController to support streaming of pre-calibrated lens data",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "LiveLinkHub" ],
	"Modules": [
		{
			"Name": "LiveLinkLens",
			"Type": "Runtime",
			"ProgramAllowlist": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "CameraCalibrationCore",
			"Enabled": true
		},
		{
			"Name": "LensComponent",
			"Enabled": true
		}
	]
}