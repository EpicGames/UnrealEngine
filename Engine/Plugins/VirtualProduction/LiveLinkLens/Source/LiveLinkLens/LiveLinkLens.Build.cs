// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkLens : ModuleRules
{
	public LiveLinkLens(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CameraCalibrationCore",
				"CinematicCamera",
				"LiveLinkComponents",
				"LiveLinkInterface"
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"LensComponent",
				"LiveLinkMovieScene",
				"MovieScene",
				"MovieSceneTracks",
			}
		);
	}
}
