// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkLensController.h"

#include "CameraCalibrationSubsystem.h"
#include "Engine/Engine.h"
#include "LensComponent.h"
#include "LiveLinkLensRole.h"
#include "LiveLinkLensTypes.h"
#include "UObject/UE5MainStreamObjectVersion.h"

void ULiveLinkLensController::OnEvaluateRegistered()
{
	Super::OnEvaluateRegistered();
}

void ULiveLinkLensController::SetAttachedComponent(UActorComponent* ActorComponent)
{
	Super::SetAttachedComponent(ActorComponent);
}

void ULiveLinkLensController::Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData)
{
	const FLiveLinkLensStaticData* StaticData = SubjectData.StaticData.Cast<FLiveLinkLensStaticData>();
	const FLiveLinkLensFrameData* FrameData = SubjectData.FrameData.Cast<FLiveLinkLensFrameData>();

	if (StaticData && FrameData)
	{
		if (ULensComponent* const LensComponent = Cast<ULensComponent>(GetAttachedComponent()))
		{
			if (LensComponent->GetDistortionSource() == EDistortionSource::LiveLinkLensSubject)
			{
				UCameraCalibrationSubsystem* const SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
				const TSubclassOf<ULensModel> LensModel = SubSystem->GetRegisteredLensModel(StaticData->LensModel);

				if (LensComponent->GetLensModel() != LensModel)
				{
					LensComponent->SetLensModel(LensModel);
				}

				// Update the lens distortion handler with the latest frame of data from the LiveLink source
				FLensDistortionState DistortionState;

				DistortionState.DistortionInfo.Parameters = FrameData->DistortionParameters;
				DistortionState.FocalLengthInfo.FxFy = FrameData->FxFy;
				DistortionState.ImageCenter.PrincipalPoint = FrameData->PrincipalPoint;

				LensComponent->SetDistortionState(DistortionState);
			}
		}
	}
}

void ULiveLinkLensController::Cleanup()
{
	if (ULensComponent* const LensComponent = Cast<ULensComponent>(GetAttachedComponent()))
	{
		// If the lens component is currently receiving its distortion state from this controller, reset that state to zero
		if (LensComponent->GetDistortionSource() == EDistortionSource::LiveLinkLensSubject)
		{
			LensComponent->ClearDistortionState();
		}
	}
}

bool ULiveLinkLensController::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport)
{
	return RoleToSupport == ULiveLinkLensRole::StaticClass();
}

TSubclassOf<UActorComponent> ULiveLinkLensController::GetDesiredComponentClass() const
{
	return ULensComponent::StaticClass();
}

void ULiveLinkLensController::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	const int32 UE5MainVersion = GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID);
	if (UE5MainVersion < FUE5MainStreamObjectVersion::LensComponentDistortion)
	{
		// Distortion evaluation has moved to the Lens component. In order for existing camera actors to continue functioning as before,
		// we have to migrate some properties from the lens controller to the lens component. If the owning actor does not have already have a lens 
		// component then we will instance a new one.
		AActor* const CameraActor = GetOuterActor();
		CameraActor->ConditionalPostLoad();
		if (CameraActor->GetWorld())
		{
			TInlineComponentArray<ULensComponent*> LensComponents;
			CameraActor->GetComponents(LensComponents);

			// Loop through any existing lens components to find one whose distortion source was this lens controller
			bool bFoundMatchingLensComponent = false;
			for (ULensComponent* LensComponent : LensComponents)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FDistortionHandlerPicker DeprecatedPicker = LensComponent->GetDistortionHandlerPicker();
				if (DeprecatedPicker.DistortionProducerID == DistortionProducerID_DEPRECATED)
				{
					bFoundMatchingLensComponent = true;

					// Preserve the previous "bApplyDistortion" setting from the existing component
					const bool bPreviousDistortionSetting = LensComponent->ShouldApplyDistortion();
					SetAttachedComponent(LensComponent);
					LensComponent->SetApplyDistortion(bPreviousDistortionSetting);
					
					// Copy the overscan multiplier settings to the lens component
					if (bScaleOverscan_DEPRECATED)
					{
						LensComponent->SetOverscanMultiplier(OverscanMultiplier_DEPRECATED);
					}
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			// If no matching existing lens components were found, create a new one and initialize it with the correct LensFile
			if (!bFoundMatchingLensComponent)
			{
				ULensComponent* LensComponent = NewObject<ULensComponent>(CameraActor, MakeUniqueObjectName(this, ULensComponent::StaticClass(), TEXT("Lens")));
				CameraActor->AddInstanceComponent(LensComponent);
				LensComponent->RegisterComponent();

				SetAttachedComponent(LensComponent);

				// If there was no existing lens component, then the camera would not have had distortion (from this controller) applied to it
				LensComponent->SetApplyDistortion(false);

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				// Copy the overscan multiplier settings to the lens component
				if (bScaleOverscan_DEPRECATED)
				{
					LensComponent->SetOverscanMultiplier(OverscanMultiplier_DEPRECATED);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}
#endif //WITH_EDITOR
}
