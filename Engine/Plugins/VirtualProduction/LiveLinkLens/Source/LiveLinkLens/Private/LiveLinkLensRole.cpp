// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkLensRole.h"
#include "LiveLinkLensTypes.h"

#define LOCTEXT_NAMESPACE "LiveLinkRole"

UScriptStruct* ULiveLinkLensRole::GetStaticDataStruct() const
{
	return FLiveLinkLensStaticData::StaticStruct();
}

UScriptStruct* ULiveLinkLensRole::GetFrameDataStruct() const
{
	return FLiveLinkLensFrameData::StaticStruct();
}

UScriptStruct* ULiveLinkLensRole::GetBlueprintDataStruct() const
{
	return FLiveLinkLensBlueprintData::StaticStruct();
}

bool ULiveLinkLensRole::InitializeBlueprintData(const FLiveLinkSubjectFrameData& InSourceData, FLiveLinkBlueprintDataStruct& OutBlueprintData) const
{
	bool bSuccess = false;

	FLiveLinkLensBlueprintData* BlueprintData = OutBlueprintData.Cast<FLiveLinkLensBlueprintData>();
	const FLiveLinkLensStaticData* StaticData = InSourceData.StaticData.Cast<FLiveLinkLensStaticData>();
	const FLiveLinkLensFrameData* FrameData = InSourceData.FrameData.Cast<FLiveLinkLensFrameData>();
	if (BlueprintData && StaticData && FrameData)
	{
		GetStaticDataStruct()->CopyScriptStruct(&BlueprintData->StaticData, StaticData);
		GetFrameDataStruct()->CopyScriptStruct(&BlueprintData->FrameData, FrameData);
		bSuccess = true;
	}

	return bSuccess;
}

FText ULiveLinkLensRole::GetDisplayName() const
{
	return LOCTEXT("LensRole", "Lens");
}

#undef LOCTEXT_NAMESPACE