// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneLiveLinkSubSectionLensRole.h"

#include "CameraCalibrationSubsystem.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Engine/Engine.h"
#include "Channels/MovieSceneByteChannel.h"
#include "LiveLinkLensRole.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "LiveLinkLensTypes.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "Models/LensModel.h"
#include "MovieScene/IMovieSceneLiveLinkPropertyHandler.h"
#include "MovieScene/MovieSceneLiveLinkStructProperties.h"

void UMovieSceneLiveLinkSubSectionLensRole::Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData)
{
	Super::Initialize(InSubjectRole, InStaticData);

	CreatePropertiesChannel();
}

int32 UMovieSceneLiveLinkSubSectionLensRole::CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData)
{
	int32 StartIndex = InChannelIndex;
	InChannelIndex = 0;

	// Get the lens model from the live link static data
	ULensModel* LensModel = nullptr;
	FLiveLinkLensStaticData* LensData = StaticData->Cast<FLiveLinkLensStaticData>();
	check(LensData);

	UCameraCalibrationSubsystem* const SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	if (SubSystem)
	{
		TSubclassOf<ULensModel> LensModelClass = SubSystem->GetRegisteredLensModel(LensData->LensModel);

		if (LensModelClass.Get())
		{
			LensModel = LensModelClass->GetDefaultObject<ULensModel>();
		}
	}

	if (LensModel)
	{
		const int32 ParameterCount = LensModel->GetNumParameters();

#if WITH_EDITOR
		TArray<FText> ParameterDisplayNames = LensModel->GetParameterDisplayNames();
		check(ParameterDisplayNames.Num() == ParameterCount);
#endif //#WITH_EDITOR

		for (int32 Index = 0; Index < ParameterCount; ++Index)
		{
#if WITH_EDITOR
			MovieSceneLiveLinkSectionUtils::CreateChannelEditor(ParameterDisplayNames[Index], SubSectionData.Properties[0].FloatChannel[InChannelIndex], StartIndex + InChannelIndex, TMovieSceneExternalValue<float>(), OutChannelMask, OutChannelData);
#else
			OutChannelData.Add(SubSectionData.Properties[0].FloatChannel[InChannelIndex]);
#endif //#WITH_EDITOR

			++InChannelIndex;
		}
	}

	return InChannelIndex;
}

void UMovieSceneLiveLinkSubSectionLensRole::CreatePropertiesChannel()
{
	// Get the lens model from the live link static data
	ULensModel* LensModel = nullptr;
	FLiveLinkLensStaticData* LensData = StaticData->Cast<FLiveLinkLensStaticData>();
	check(LensData);

	UCameraCalibrationSubsystem* const SubSystem = GEngine->GetEngineSubsystem<UCameraCalibrationSubsystem>();
	if (SubSystem)
	{
		TSubclassOf<ULensModel> LensModelClass = SubSystem->GetRegisteredLensModel(LensData->LensModel);

		if (LensModelClass.Get())
		{
			LensModel = LensModelClass->GetDefaultObject<ULensModel>();
		}
	}

	if (LensModel)
	{
		const int32 ParameterCount = LensModel->GetNumParameters();
		if (ParameterCount > 0)
		{
			const FName PropertyName = GET_MEMBER_NAME_CHECKED(FLiveLinkLensFrameData, DistortionParameters);
			SubSectionData.Properties.SetNum(1);
			SubSectionData.Properties[0].PropertyName = PropertyName;
			PropertyHandler = LiveLinkPropertiesUtils::CreatePropertyHandler(*FLiveLinkLensFrameData::StaticStruct(), &SubSectionData.Properties[0]);
			PropertyHandler->CreateChannels(*FLiveLinkLensFrameData::StaticStruct(), ParameterCount);
		}
	}
}

void UMovieSceneLiveLinkSubSectionLensRole::RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData)
{
	const FLiveLinkLensFrameData* LensFrameData = InFrameData.Cast<FLiveLinkLensFrameData>();
	check(LensFrameData);

	if (PropertyHandler.IsValid())
	{
		PropertyHandler->RecordFrame(InFrameNumber, *FLiveLinkLensFrameData::StaticStruct(), InFrameData.GetBaseData());
	}
}

void UMovieSceneLiveLinkSubSectionLensRole::FinalizeSection(bool bInReduceKeys, const FKeyDataOptimizationParams& InOptimizationParams)
{
	if (PropertyHandler.IsValid())
	{
		PropertyHandler->Finalize(bInReduceKeys, InOptimizationParams);
	}
}

bool UMovieSceneLiveLinkSubSectionLensRole::IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const
{
	return (RoleToSupport->IsChildOf(ULiveLinkLensRole::StaticClass()));
}
