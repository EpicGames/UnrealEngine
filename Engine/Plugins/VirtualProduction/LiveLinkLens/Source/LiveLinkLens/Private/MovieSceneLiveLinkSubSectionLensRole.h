// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene/MovieSceneLiveLinkSubSection.h"


#include "MovieSceneLiveLinkSubSectionLensRole.generated.h"


/**
 * A LiveLinkSubSection managing special properties of the LensRole
 */
UCLASS()
class LIVELINKLENS_API UMovieSceneLiveLinkSubSectionLensRole : public UMovieSceneLiveLinkSubSection
{
	GENERATED_BODY()

public:
	virtual void Initialize(TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData) override;
	virtual int32 CreateChannelProxy(int32 InChannelIndex, TArray<bool>& OutChannelMask, FMovieSceneChannelProxyData& OutChannelData) override;
	virtual void RecordFrame(FFrameNumber InFrameNumber, const FLiveLinkFrameDataStruct& InFrameData) override;
	virtual void FinalizeSection(bool bReduceKeys, const FKeyDataOptimizationParams& OptimizationParams) override;

public:
	virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) const override;

private:
	void CreatePropertiesChannel();

protected:
	/** Helper struct to manage filling channels from distortion parameter array */
	TSharedPtr<IMovieSceneLiveLinkPropertyHandler> PropertyHandler;

};
