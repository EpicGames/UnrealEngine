// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkControllerBase.h"

#include "LensDistortionModelHandlerBase.h"

#include "LiveLinkLensController.generated.h"

/**
 * LiveLink Controller for the LensRole to drive lens distortion data 
 */
UCLASS()
class LIVELINKLENS_API ULiveLinkLensController : public ULiveLinkControllerBase
{
	GENERATED_BODY()

public:
	//~ Begin ULiveLinkControllerBase interface
	virtual void OnEvaluateRegistered() override;
	virtual void Tick(float DeltaTime, const FLiveLinkSubjectFrameData& SubjectData) override;
	virtual bool IsRoleSupported(const TSubclassOf<ULiveLinkRole>& RoleToSupport) override;
	virtual TSubclassOf<UActorComponent> GetDesiredComponentClass() const override;
	virtual void SetAttachedComponent(UActorComponent* ActorComponent) override;
	virtual void Cleanup() override;
	//~ End ULiveLinkControllerBase interface

	//~ Begin UObject interface
	virtual void PostLoad() override;
	//~ End UObject interface

protected:

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.1, "This property has been deprecated. Distortion is handled directly by controlled Lens Component.")
 	UPROPERTY(Transient)
 	TObjectPtr<ULensDistortionModelHandlerBase> LensDistortionHandler_DEPRECATED = nullptr;

	UE_DEPRECATED(5.1, "This property has been deprecated and is no longer used.")
	UPROPERTY(DuplicateTransient)
	FGuid DistortionProducerID_DEPRECATED;

	UE_DEPRECATED(5.1, "This property has been deprecated. Overscan multipliers should now be applied from a Lens Component.")
	UPROPERTY()
	bool bScaleOverscan_DEPRECATED = false;

	UE_DEPRECATED(5.1, "This property has been deprecated. Overscan multipliers should now be applied from a Lens Component.")
	UPROPERTY()
	float OverscanMultiplier_DEPRECATED = 1.0f;
#endif //WITH_EDITORONLY_DATA
};
