// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Roles/LiveLinkCameraTypes.h"

#include "LensData.h"
#include "Models/LensModel.h"

#include "LiveLinkLensTypes.generated.h"

/**
 * Struct for static lens data
 */
USTRUCT(BlueprintType)
struct LIVELINKLENS_API FLiveLinkLensStaticData : public FLiveLinkCameraStaticData
{
	GENERATED_BODY()

	/** Specifies the type/character of the lens (spherical, anamorphic, etc.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FName LensModel;
};

/**
 * Struct for dynamic (per-frame) lens data
 */
USTRUCT(BlueprintType)
struct LIVELINKLENS_API FLiveLinkLensFrameData : public FLiveLinkCameraFrameData
{
	GENERATED_BODY()

	/** Parameters used by the distortion model */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	TArray<float> DistortionParameters;

	/** Normalized focal length used by the distortion model */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp)
	FVector2D FxFy = FVector2D(1.0f, (16.0f / 9.0f));
	
	/** Normalized center of the image, in the range [0.0f, 1.0f] */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink", Interp, meta = (DisplayName = "Image Center"))
	FVector2D PrincipalPoint = FVector2D(0.5f, 0.5f);
};

/**
 * Facility structure to handle lens data in blueprint
 */
USTRUCT(BlueprintType)
struct LIVELINKLENS_API FLiveLinkLensBlueprintData : public FLiveLinkBaseBlueprintData
{
	GENERATED_BODY()

	/** Static data that should not change every frame */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkLensStaticData StaticData;

	/** Dynamic data that can change every frame  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "LiveLink")
	FLiveLinkLensFrameData FrameData;
};
